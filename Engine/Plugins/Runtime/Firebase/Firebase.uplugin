{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Firebase",
	"Description": "Support for remote notifications using Firebase",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Android",
		"IOS"
	],
	"Modules": [
		{
			"Name": "Firebase",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Android",
				"IOS"
			]
		}
	],
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false
}