// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class Firebase : ModuleRules
	{
		public Firebase(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			PrivateDependencyModuleNames.AddRange(new string[] {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "Launch"
            });

            PublicDefinitions.Add("WITH_FIREBASE=1");
			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);

            if (Target.Platform == UnrealTargetPlatform.Android)
			{
				PrivateDependencyModuleNames.Add("Launch");

				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "Firebase.upl.xml"));
			}
			else if (Target.Platform == UnrealTargetPlatform.IOS)
			{
                PrivateDependencyModuleNames.Add("Swift");
   
				PublicDefinitions.Add("WITH_FIREBASE_IOS=1");
				string FirebaseBaseIOSDir = Path.Combine(ModuleDirectory, "ThirdParty/IOS/");
				PublicSystemIncludePaths.AddRange(
					new string[] {
						FirebaseBaseIOSDir + "include",
					}
				);
				
				// Firebase frameworks for cloud messaging (notifications)
				PublicAdditionalFrameworks.Add(
					new Framework(
					"FirebaseAnalytics",
					Path.Combine(FirebaseBaseIOSDir, "FirebaseAnalytics.framework.zip"),
					""
					)
				);
				
				PublicAdditionalFrameworks.Add(
					new Framework(
					"FirebaseCore",
					Path.Combine(FirebaseBaseIOSDir, "FirebaseCore.framework.zip"),
					""
					)
				);
				
				PublicAdditionalFrameworks.Add(
					new Framework(
					"FirebaseMessaging",
					Path.Combine(FirebaseBaseIOSDir, "FirebaseMessaging.framework.zip"),
					""
					)
				);
				
				PublicAdditionalFrameworks.Add(
					new Framework(
					"nanopb",
					Path.Combine(FirebaseBaseIOSDir, "nanopb.framework.zip"),
					""
					)
				);
                    
                PublicAdditionalFrameworks.Add(
                    new Framework(
                    "FirebaseCoreInternal",
                    Path.Combine(FirebaseBaseIOSDir, "FirebaseCoreInternal.framework.zip"),
                    ""
                    )
                );
                                
                PublicAdditionalFrameworks.Add(
                    new Framework(
                    "FirebaseInstallations",
                    Path.Combine(FirebaseBaseIOSDir, "FirebaseInstallations.framework.zip"),
                    ""
                    )
                );
                                                                                
                PublicAdditionalFrameworks.Add(
                    new Framework(
                    "GoogleUtilities",
                    Path.Combine(FirebaseBaseIOSDir, "GoogleUtilities.framework.zip"),
                    ""
                    )
                );
                                                                                                
                PublicAdditionalFrameworks.Add(
                    new Framework(
                    "GoogleDataTransport",
                    Path.Combine(FirebaseBaseIOSDir, "GoogleDataTransport.framework.zip"),
                    ""
                    )
                );
                                                                                                                
                PublicAdditionalFrameworks.Add(
                    new Framework(
                    "FBLPromises",
                    Path.Combine(FirebaseBaseIOSDir, "FBLPromises.framework.zip"),
                    ""
                    )
                );
                                                                                                                                
                PublicAdditionalFrameworks.Add(
                    new Framework(
                    "GoogleAppMeasurement",
                    Path.Combine(FirebaseBaseIOSDir, "GoogleAppMeasurement.framework.zip"),
                    ""
                    )
                );
			}

			PublicIncludePathModuleNames.Add("Launch");
		}
	}
}
