<?xml version="1.0" encoding="utf-8"?>
<!-- Firebase plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Firebase init"/>

		<setBoolFromProperty result="bEnabled" ini="Engine" section="Firebase" property="FirebaseEnabled" default="false"/>
		<setBoolFromProperty result="bDisableAutoInit" ini="Engine" section="Firebase" property="FirebaseDisableAutoInit" default="false"/>
		<setStringFromProperty result="FirebaseService" ini="Engine" section="Firebase" property="FirebaseService" default="com.epicgames.unreal.notifications.EpicFirebaseMessagingService"/>
		<if condition="bEnabled">
			<true>
				<log text="Firebase enabled!"/>

				<!-- May need client sender ID for notification handling -->
				<!-- setStringFromProperty result="firebase-client-sender-id" ini="Engine" section="Firebase" property="FirebaseClientSenderID" default="" /-->
			</true>
		</if>
	</init>

	<registerBuildSettings>
		<insertValue value="Firebase[FirebaseEnabled,FirebaseDisableAutoInit,FirebaseService]"/>
		<insertNewline/>
	</registerBuildSettings>
	
	<androidManifestUpdates>
		<if condition="bEnabled">
			<true>
				<setElement result="ServiceElement" value="service" />
				<addAttribute tag="$ServiceElement" name="android:name" value="$S(FirebaseService)" />
				<addAttribute tag="$ServiceElement" name="android:exported" value="false" />
				<addElements tag="$ServiceElement">
					  	<intent-filter>
							<action android:name="com.google.firebase.MESSAGING_EVENT"/>
						</intent-filter>
				</addElements>
				<if condition="bDisableAutoInit">
					<true>
						<addElements tag="$ServiceElement">
							<meta-data android:name="firebase_messaging_auto_init_enabled" android:value="false" />
							<meta-data android:name="firebase_analytics_collection_enabled" android:value="false" />
						</addElements>
						<setElement result="removeFirebaseInit" value="provider" />
						<addAttribute tag="$removeFirebaseInit" name="android:name" value="com.google.firebase.provider.FirebaseInitProvider" />
						<addAttribute tag="$removeFirebaseInit" name="android:authorities" value="$S(AndroidPackageName).firebaseinitprovider" />
						<addAttribute tag="$removeFirebaseInit" name="android:exported" value="false" />
						<addAttribute tag="$removeFirebaseInit" name="tools:node" value="remove" />
						<addElement tag="application" name="removeFirebaseInit" />
					</true>
				</if>
				<addElement tag="application" name="ServiceElement" />
			</true>
		</if>
	</androidManifestUpdates>

	<buildscriptGradleAdditions>
		<if condition="bEnabled">
			<true>
				<insert>dependencies {
	classpath 'com.google.gms:google-services:4.4.2' // google-services plugin
}
</insert>
			</true>
		</if>
	</buildscriptGradleAdditions>

	<buildGradleAdditions>
		<if condition="bEnabled">
			<true>
				<!-- Needs to be same version number as play-services -->
				<insert>dependencies {
	implementation 'com.google.firebase:firebase-core:21.1.1'
	implementation 'com.google.firebase:firebase-messaging:23.0.8'
}

// init firebase
apply plugin: 'com.google.gms.google-services'
</insert>
			</true>
		</if>
	</buildGradleAdditions>

	<prebuildCopies>
		<if condition="bEnabled">
			<true>
				<copyDir src="$S(PluginDir)/Java" dst="$S(BuildDir)/src/com/epicgames/unreal"/>
			</true>
			<false>
				<deleteFiles filespec="$S(BuildDir)/src/com/epicgames/unreal/notifications/*.java"/>
			</false>
		</if>
	</prebuildCopies>
	
	<gradleCopies>
		<if condition="bEnabled">
			<true>
				<!-- Will need to revisit this if/when other plug-ins need to modify google-services.json -->
				<copyFile src="$S(BuildDir)/google-services.json" dst="$S(BuildDir)/gradle/app/google-services.json" force="false"/>
				<copyFile src="$S(BuildDir)/google-services-$S(Configuration).json" dst="$S(BuildDir)/gradle/app/google-services.json" force="false"/>
			</true>
		</if>
	</gradleCopies>

	<gameActivityAllowedRemoteNotificationsAdditions>
		<if condition="bEnabled">
			<true>
				<insert>		bPluginEnabled = true;</insert>
				<insertNewline/>
			</true>
		</if>
	</gameActivityAllowedRemoteNotificationsAdditions>
</root>
