// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicFirebaseIOSNotifications.h"

#if PLATFORM_IOS && WITH_IOS_FIREBASE_INTEGRATION

THIRD_PARTY_INCLUDES_START
_Pragma("clang diagnostic push")
_Pragma("clang diagnostic ignored \"-Wobjc-property-no-attribute\"")
#include "ThirdParty/IOS/include/Firebase.h"
_Pragma("clang diagnostic pop")
THIRD_PARTY_INCLUDES_END
// Module header, needed for log category
#include "Firebase.h"
#include "IOS/IOSAppDelegate.h"

#include "Misc/Paths.h"
#include "Async/TaskGraphInterfaces.h"

bool FFirebaseIOSNotifications::bIsInitialized = false;
FString FFirebaseIOSNotifications::IOSFirebaseToken;
NSString* KEY_FIREBASE_TOKEN = @"firebasetoken";
NSString* KEY_FIREBASE_PROJECT_ID = @"firebaseprojectid";

@interface IOSAppDelegate (FirebaseHandling) <FIRMessagingDelegate>

-(void)SetupFirebase : (Boolean) enableAnalytics;
-(void)ScheduleFirebaseTokenUpdate;

@end

@implementation IOSAppDelegate (FirebaseHandling)

-(void)SetupFirebase : (Boolean) enableAnalytics
{
	if (enableAnalytics)
	{
		[FIRAnalytics setAnalyticsCollectionEnabled:YES];
	}
    [FIRMessaging messaging].delegate = self;
    [UNUserNotificationCenter currentNotificationCenter].delegate = self;
    
	
	UNAuthorizationOptions authOptions = UNAuthorizationOptionAlert | UNAuthorizationOptionSound | UNAuthorizationOptionBadge;
	[[UNUserNotificationCenter currentNotificationCenter] requestAuthorizationWithOptions:authOptions
		completionHandler:^(BOOL granted, NSError * _Nullable error) 
	 {
		if (granted)
		{
			UE_LOGF(LogFirebase, Log, "Firebase authorization granted");
			
			[[UIApplication sharedApplication] registerForRemoteNotifications];
		}
		else
		{
			UE_LOGF(LogFirebase, Log, "Firebase authorization denied");
		}
	}];
}

-(void)ConfigureFirebase
{
    [FIRApp configure];
	
#if !UE_BUILD_SHIPPING
	NSString* currentProjectID = [[[FIRApp defaultApp] options] GCMSenderID];
	NSLog(@"Firebase configured for project %@", currentProjectID);
#endif
}

-(void)ConfigureFirebaseWithCustomFile : (NSString*)fileName fileExtension:(NSString*)ext
{
	NSString* filePath = [[NSBundle mainBundle] pathForResource:fileName ofType:ext];
	if (filePath == nil)
	{
		UE_LOGF(LogFirebase, Warning, "Failed to find custom Firebase file, using default configuration");
		[self ConfigureFirebase];
	}
	else
	{
		FIROptions* options = [[FIROptions alloc] initWithContentsOfFile:filePath];
		if (options == nil)
		{
			UE_LOGF(LogFirebase, Warning, "Failed to parse custom Firebase options, using default configuration");
			[self ConfigureFirebase];
		}
		else
		{
			[FIRApp configureWithOptions:options];
			
#if !UE_BUILD_SHIPPING
	NSString* currentProjectID = [[[FIRApp defaultApp] options] GCMSenderID];
	NSLog(@"Firebase configured using custom config for project %@", currentProjectID);
#endif
		}
	}
}

-(void)messaging:(FIRMessaging *)messaging didReceiveRegistrationToken:(NSString *)fcmToken 
{
	if (fcmToken == nil)
	{
		// happens when deleting the token
		return;
	}
	
    NSDictionary *dataDict = [NSDictionary dictionaryWithObject:fcmToken forKey:@"token"];
    [[NSNotificationCenter defaultCenter] postNotificationName:
     @"FCMToken" object:nil userInfo:dataDict];
    
    FString Token = FString(fcmToken);
    FFirebaseIOSNotifications::SetFirebaseToken(Token);
#if !UE_BUILD_SHIPPING
    UE_LOGF(LogFirebase, Log, "Firebase Token Refreshed : %ls", *Token);
#endif
	NSString* projectID = [[[FIRApp defaultApp] options] GCMSenderID];
    [[NSUserDefaults standardUserDefaults] setObject:fcmToken forKey:KEY_FIREBASE_TOKEN];
	[[NSUserDefaults standardUserDefaults] setObject:projectID forKey:KEY_FIREBASE_PROJECT_ID];
    [[NSUserDefaults standardUserDefaults] synchronize];
}

- (void)ScheduleFirebaseTokenUpdate
{
    NSUserDefaults* UserDefaults = [NSUserDefaults standardUserDefaults];
	NSString* currentProjectID = [[[FIRApp defaultApp] options] GCMSenderID];
	if ([UserDefaults objectForKey:KEY_FIREBASE_PROJECT_ID] == nil ||
		![[UserDefaults stringForKey:KEY_FIREBASE_PROJECT_ID] isEqualToString:currentProjectID])
	{
		if ([UserDefaults objectForKey:KEY_FIREBASE_TOKEN] != nil)
		{
			UE_LOGF(LogFirebase, Log, "Firebase project changed, removing token");
			[UserDefaults removeObjectForKey:KEY_FIREBASE_TOKEN];
		}
	}
	
    if ([UserDefaults objectForKey:KEY_FIREBASE_TOKEN] != nil)
    {
        FString Token = FString([UserDefaults stringForKey:KEY_FIREBASE_TOKEN]);
        FFirebaseIOSNotifications::SetFirebaseToken(Token);
#if !UE_BUILD_SHIPPING
        UE_LOGF(LogFirebase, Log, "Retrieved Firebase Token from cache : %ls", *Token);
#endif
    }
    
	// Query token from Firebase even if there is one in cache already
	
	
	[[FIRMessaging messaging] tokenWithCompletion:^(NSString *firebaseToken, NSError *error)
	 {
		if (error == nil && firebaseToken != nil)
		{
			FString Token = FString(firebaseToken);
			FFirebaseIOSNotifications::SetFirebaseToken(Token);
#if !UE_BUILD_SHIPPING
			UE_LOGF(LogFirebase, Log, "Firebase Token Queried : %ls", *Token);
#endif
			NSString* projectID = [[[FIRApp defaultApp] options] GCMSenderID];
			[[NSUserDefaults standardUserDefaults] setObject:firebaseToken forKey:KEY_FIREBASE_TOKEN];
			[[NSUserDefaults standardUserDefaults] setObject:projectID forKey:KEY_FIREBASE_PROJECT_ID];
			[[NSUserDefaults standardUserDefaults] synchronize];
		}
	}];
}
@end

void FFirebaseIOSNotifications::ConfigureFirebase()
{
    if (!IsConfigured())
    {
        [[IOSAppDelegate GetDelegate] ConfigureFirebase];
    }
}

void FFirebaseIOSNotifications::ConfigureFirebaseWithCustomFile(const FString& FileName)
{
	if (IsConfigured())
	{
		UE_LOGF(LogFirebase, Warning, "Failed to configure Firebase with custom file. Firebase is already configured!");
		return;
	}
	
	FString Name = FPaths::GetBaseFilename(FileName);
	FString Extension = FPaths::GetExtension(FileName);
	CFStringRef NameStr = FPlatformString::TCHARToCFString(*Name);
	CFStringRef ExtensionStr = FPlatformString::TCHARToCFString(*Extension);
	[[IOSAppDelegate GetDelegate] ConfigureFirebaseWithCustomFile:(NSString*)NameStr fileExtension:(NSString*)ExtensionStr];
}

void FFirebaseIOSNotifications::Initialize(uint64 TokenQueryTimeoutNanoseconds, bool bEnableAnalytics)
{
    if (!IsConfigured())
    {
        ConfigureFirebase();
    }
    
    if (!bIsInitialized)
    {
        [[IOSAppDelegate GetDelegate] SetupFirebase:bEnableAnalytics];
        bIsInitialized = true;
    }
	else 
	{
		// Ensure we still have a valid token if already initialized
		[[IOSAppDelegate GetDelegate] ScheduleFirebaseTokenUpdate];
	}
}

void FFirebaseIOSNotifications::SetFirebaseToken(FString Token)
{
    @synchronized ([IOSAppDelegate GetDelegate])
    {
		if (IFirebaseModuleInterface::Get().OnTokenUpdate.IsBound())
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady([OldToken = IOSFirebaseToken, NewToken = Token]()
				{
					IFirebaseModuleInterface::Get().OnTokenUpdate.Broadcast(OldToken, NewToken);
				}, TStatId(), NULL, ENamedThreads::GameThread);
		}
        IOSFirebaseToken = Token;
    }
}

FString FFirebaseIOSNotifications::GetFirebaseToken()
{
    FString Token;
    @synchronized ([IOSAppDelegate GetDelegate])
    {
        Token = IOSFirebaseToken;
    }
    
    return Token;
}

void FFirebaseIOSNotifications::EnableFirebaseAutoInit()
{
	[FIRMessaging messaging].autoInitEnabled = YES;
}

bool FFirebaseIOSNotifications::IsConfigured()
{
	return [[FIRApp allApps] count] != 0;
}

void FFirebaseIOSNotifications::DeleteFirebaseToken()
{
	if (!IsConfigured())
	{
		UE_LOGF(LogFirebase, Warning, "Trying to delete Firebase token, but Firebase is not configured.")
		return;
	}
	
	[[NSUserDefaults standardUserDefaults] removeObjectForKey:KEY_FIREBASE_TOKEN];
	[[NSUserDefaults standardUserDefaults] synchronize];
	
	[[FIRMessaging messaging] deleteTokenWithCompletion:^(NSError *error)
	 {
#if !UE_BUILD_SHIPPING
		NSLog(@"Error when deleting Firebase token: %@", error);
#endif
	}];
}

void FFirebaseIOSNotifications::OnAPNSTokenReceived(TArrayView<uint8> Token)
{
	NSData* apnsToken = Token.IsEmpty() ? nil : [NSData dataWithBytes:Token.GetData() length:Token.Num()];
	
	if (apnsToken != nil)
	{
		[[FIRMessaging messaging] setAPNSToken:apnsToken];
	}
	
	[[IOSAppDelegate GetDelegate] ScheduleFirebaseTokenUpdate];
}

#endif // PLATFORM_IOS
