// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifndef WITH_IOS_FIREBASE_INTEGRATION
#define WITH_IOS_FIREBASE_INTEGRATION 0
#endif

#if PLATFORM_IOS && WITH_IOS_FIREBASE_INTEGRATION

struct FIREBASE_API FFirebaseIOSNotifications
{
private:
    static bool bIsInitialized;
    static bool bIsConfigured;
    static FString IOSFirebaseToken;
	
	static bool IsConfigured();
    
public:
    static void ConfigureFirebase();
	static void ConfigureFirebaseWithCustomFile(const FString& FileName);
	static void EnableFirebaseAutoInit();
    static void Initialize(uint64 TokenQueryTimeoutNanoseconds, bool bEnableAnalytics);
    static void SetFirebaseToken(FString Token);
    static FString GetFirebaseToken();
	static void DeleteFirebaseToken();
	static void OnAPNSTokenReceived(TArrayView<uint8> Token);
};

#endif // PLATFORM_IOS
