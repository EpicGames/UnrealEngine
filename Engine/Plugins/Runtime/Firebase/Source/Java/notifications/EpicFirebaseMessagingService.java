package com.epicgames.unreal.notifications;

import android.app.NotificationChannel;
import android.app.NotificationManager;
import android.app.PendingIntent;
import android.content.Context;
import android.content.Intent;
import android.net.Uri;
import android.os.Build;
import android.content.SharedPreferences;
import androidx.annotation.NonNull;
import androidx.annotation.Nullable;
import androidx.core.app.NotificationCompat;
import android.text.TextUtils;

import com.epicgames.unreal.GameActivity;
import com.epicgames.unreal.GameApplication;
import com.epicgames.unreal.LocalNotificationReceiver;
import com.epicgames.unreal.Logger;
import com.google.android.gms.tasks.OnCompleteListener;
import com.google.android.gms.tasks.Task;
import com.google.android.gms.tasks.Tasks;
import com.google.firebase.FirebaseApp;
import com.google.firebase.FirebaseOptions;
import com.google.firebase.messaging.FirebaseMessaging;
import com.google.firebase.messaging.FirebaseMessagingService;
import com.google.firebase.messaging.RemoteMessage;
import com.google.firebase.analytics.FirebaseAnalytics;

import org.json.JSONException;
import org.json.JSONObject;

import java.util.Map;
import java.util.Objects;
import java.util.Random;
import java.io.InputStreamReader;
import java.io.BufferedReader;
import java.io.InputStream;
import java.util.stream.Collectors;

import static com.epicgames.unreal.GameActivity.Get;
import static com.epicgames.unreal.GameActivity.LocalNotificationGetID;
import static com.epicgames.unreal.LocalNotificationReceiver.KEY_LOCAL_NOTIFICATION_ACTION;
import static com.epicgames.unreal.LocalNotificationReceiver.KEY_LOCAL_NOTIFICATION_BODY;
import static com.epicgames.unreal.LocalNotificationReceiver.KEY_LOCAL_NOTIFICATION_ID;
import static com.epicgames.unreal.LocalNotificationReceiver.KEY_LOCAL_NOTIFICATION_TITLE;

public class EpicFirebaseMessagingService extends FirebaseMessagingService {

	private static final Logger Log = new Logger("UE-" + EpicFirebaseMessagingService.class.getSimpleName());

	private static final String ATTR_TYPE = "type";
	private static final String PAYLOAD_P_KEY = "p";

	private static final String FIREBASE_ICON_NAME = "ic_notification";
	private static final String FIREBASE_ICON_TYPE = "mipmap";

	private static final String COMPONENT = "MessagingService";
	private static final String SYSTEM = "Notification";
	private static final String INCOMING_MESSAGE_ERROR = "IncomingMessageError";
	private static final String INCOMING_MESSAGE_WARNING = "IncomingMessageWarning";
	private static final String ATTR_ERROR = "Error";
	private static final String ATTR_WARNING = "Warning";

	// Firebase intent keys
	public static final String NOTIFICATION_ACTION = "ue4_fb.notificationAction";
	public static final String KEY_PUSH_NOTIFICATION = "ue4_fb.push";
	public static final String KEY_NOTIFICATION_BODY = "ue4_fb.body";
	private static final String KEY_MESSAGE_ID = "ue4_fb.messageId";
	private static final String KEY_NOTIFICATION_TYPE = "ue4_fb.type";

	public static native void OnFirebaseTokenChange(String PreviousToken, String NewToken);

	@Override
	public void onMessageReceived(@NonNull RemoteMessage remoteMessage) {
		Log.verbose("Firebase onMessageReceived");
		super.onMessageReceived(remoteMessage);
		final String messageId = remoteMessage.getMessageId();
		final Map<String, String> data = remoteMessage.getData();
		logNotificationInfo(remoteMessage.getNotification());
		if (data != null) {
			EpicFirebaseNotificationMeta meta = null;
			try {
				meta = getContentInfo(messageId, data);
			} catch (JSONException e) {
				Log.error("Firebase unable to parse payload", e);
			}
			if (meta != null && messageId != null) {
				doNotify(messageId, meta);
			} else {
				Log.debug("containerType is empty");
				sendLocalNotification(remoteMessage);
			}
		}
	}

	private void doNotify(String messageId, EpicFirebaseNotificationMeta meta) {
		createChannel(meta);
		Intent intent = getDefaultIntent(messageId, meta);
		NotificationCompat.Builder notificationBuilder = getNotificationBuilder(meta, intent);
		NotificationManager notificationManager;
		if (Build.VERSION.SDK_INT >= Build.VERSION_CODES.M) {
			notificationManager = getSystemService(NotificationManager.class);
		} else {
			notificationManager = (NotificationManager) getSystemService(NOTIFICATION_SERVICE);
		}
		if (notificationManager != null) {
			if (GameApplication.isAppInBackground()) {
				notificationManager.notify(new Random(System.currentTimeMillis()).nextInt(), notificationBuilder.build());
				Log.verbose("Push notification notify");
			} else {
				this.startActivity(intent);
				Log.verbose("Push notification sent to activity while app is in the foreground");
			}
		}
	}

	@NonNull
	private NotificationCompat.Builder getNotificationBuilder(@NonNull EpicFirebaseNotificationMeta meta, @NonNull Intent defaultIntent) {
		return new NotificationCompat.Builder(this, meta.getChannelType())
			.setSmallIcon(meta.getNotificationResId())
			.setContentTitle(meta.getTitle())
			.setContentText(meta.getMessage())
			.setContentInfo(meta.getContentInfo())
			.setContentIntent(getPendingIntentIntent(defaultIntent))
			.setAutoCancel(true)
			.setPriority(NotificationCompat.PRIORITY_DEFAULT);
	}

	@Nullable
	private EpicFirebaseNotificationMeta getContentInfo(@Nullable String messageId, @NonNull Map<String, String> messageData) throws JSONException {
		@Nullable EpicFirebaseNotificationMeta meta = null;
		String payload = messageData.get(PAYLOAD_P_KEY);
		logMessageData(messageData);
		if (!TextUtils.isEmpty(payload)) {
			String type = getType(payload);
			Log.verbose("Message Received ( " + messageId + ") type = " + type + " : " + payload);

//			meta = new EpicFirebaseNotificationMeta();
		} else {
			Log.warn("Firebase message received ( " + messageId + ") - NO PAYLOAD");
		}
		return meta;
	}

	private void logNotificationInfo(@Nullable RemoteMessage.Notification notification) {
		if(notification == null) {
			Log.verbose("Firebase no notification data");
		} else {
			JSONObject notificationData = new JSONObject();
			try {
				notificationData.put("title", getSafeString(notification.getTitle()));
				notificationData.put("body", getSafeString(notification.getBody()));
				notificationData.put("body_loc_key", getSafeString(notification.getBodyLocalizationKey()));
				notificationData.put("click_action", getSafeString(notification.getClickAction()));
				notificationData.put("color", getSafeString(notification.getColor()));
				notificationData.put("icon", getSafeString(notification.getIcon()));
				notificationData.put("sound", getSafeString(notification.getSound()));
				notificationData.put("tag", getSafeString(notification.getTag()));
				notificationData.put("title_loc_key", getSafeString(notification.getTitleLocalizationKey()));
				String [] bodyArgs = notification.getBodyLocalizationArgs();
				if(bodyArgs != null) {
					int i = 0;
					for(String arg : bodyArgs) {
						notificationData.put("bodyArg" + i, getSafeString(arg));
						i++;
					}
				}
				Uri link = notification.getLink();
				if(link != null) {
					notificationData.put("title_loc_key", getSafeString(link.toString()));
				}
				String [] titleArgs = notification.getTitleLocalizationArgs();
				if(titleArgs != null) {
					int i = 0;
					for(String arg : titleArgs) {
						notificationData.put("titleArg" + i, getSafeString(arg));
						i++;
					}
				}
				
			} catch (Exception e) {
				Log.error("Unable to log notification", e);
			}
			Log.verbose("Firebase Notification data " + getSafeString(notificationData.toString()));
		}
	}
	
	private void logMessageData(Map<String, String> messageData) {
		JSONObject jsonObject = new JSONObject();
		for (Map.Entry<String, String> entry : messageData.entrySet()) {
			if (!"p".equals(entry.getKey())) {
				try {
					jsonObject.put(entry.getKey(), entry.getValue());
				} catch (JSONException e) {
					Log.error("Unable to add key:" + getSafeString(entry.getKey()) + " value:" + getSafeString(entry.getValue()));
				}
			}
		}
		Log.verbose("Firebase notification meta: " + jsonObject.toString());
	}

	public static String getSafeString(@Nullable String string) {
		return Objects.toString(string, "<null>");
	}

	public static int getNotificationIconId(@NonNull Context context) {
		int notificationIconID = context.getResources().getIdentifier(FIREBASE_ICON_NAME, FIREBASE_ICON_TYPE, context.getPackageName());
		if (notificationIconID == 0) {
			notificationIconID = LocalNotificationReceiver.getNotificationIconID(context);
		}
		return notificationIconID;
	}

	@NonNull
	private Intent getDefaultIntent(@NonNull String messageId, @NonNull EpicFirebaseNotificationMeta meta) {
		Intent defaultIntent = new Intent(this, GameActivity.class);
		defaultIntent.setAction(NOTIFICATION_ACTION);
		defaultIntent.setFlags(Intent.FLAG_ACTIVITY_CLEAR_TOP | Intent.FLAG_ACTIVITY_SINGLE_TOP);
		defaultIntent.putExtra(KEY_MESSAGE_ID, messageId);
		defaultIntent.putExtra(KEY_PUSH_NOTIFICATION, true);
		defaultIntent.putExtra(KEY_NOTIFICATION_TYPE, meta.getType());
		defaultIntent.putExtra(KEY_NOTIFICATION_BODY, meta.getPayload());
		return defaultIntent;
	}


	@NonNull
	private PendingIntent getPendingIntentIntent(@NonNull Intent defaultIntent) {
		return PendingIntent.getActivity(this, 1, defaultIntent, PendingIntent.FLAG_UPDATE_CURRENT | PendingIntent.FLAG_IMMUTABLE);
	}

	public void createChannel(@NonNull EpicFirebaseNotificationMeta meta) {
		if (Build.VERSION.SDK_INT >= Build.VERSION_CODES.O) {
			NotificationManager notificationManager = getSystemService(NotificationManager.class);
			if (notificationManager != null) {
				NotificationChannel channel = notificationManager.getNotificationChannel(meta.getChannelType());
				if (channel == null) {
					int importance = NotificationManager.IMPORTANCE_DEFAULT;
					channel = new NotificationChannel(meta.getChannelType(), meta.getChannelTitle(), importance);
					channel.setDescription(meta.getChannelDescription());
					notificationManager.createNotificationChannel(channel);
				}
			}
		}
	}

	private void sendLocalNotification(@NonNull RemoteMessage remoteMessage) {
		RemoteMessage.Notification notification = remoteMessage.getNotification();
		if (notification != null) {
			Intent notificationIntent = new Intent(this, LocalNotificationReceiver.class);
			int notificationID = LocalNotificationGetID(this);
			notificationIntent.putExtra(KEY_LOCAL_NOTIFICATION_ID, notificationID);
			notificationIntent.putExtra(KEY_LOCAL_NOTIFICATION_TITLE, notification.getTitle());
			notificationIntent.putExtra(KEY_LOCAL_NOTIFICATION_BODY, notification.getBody());
			notificationIntent.putExtra(KEY_LOCAL_NOTIFICATION_ACTION, notification.getClickAction());
			sendBroadcast(notificationIntent);
		}
	}

	@NonNull
	static String getType(@NonNull String payload) throws JSONException {
		JSONObject o = new JSONObject(payload);
		return o.optString(ATTR_TYPE);
	}

	// Formally in EpicFirebaseInstanceIDService.java
	private static final String PREFS_FILE_FIREBASE = "com.epicgames.firebase";
	private static final String KEY_FIREBASE_TOKEN = "firebasetoken";
	private static final String KEY_FIREBASE_PROJECT_ID = "firebaseProjectID";
	private static final String KEY_IS_UPDATED_TOKEN = "isUpdatedToken";
	private static final String KEY_IS_REGISTERED = "isRegistered";

	@Override
	public void onNewToken(@NonNull String firebaseToken) {
		Log.debug("Refreshed Firebase token: " + firebaseToken);
		if (TextUtils.isEmpty(firebaseToken)) {
			Log.error("Firebase token is empty or null");
		} else {
			saveFirebaseToken(this, firebaseToken);
			setLastFirebaseProjectID(this, FirebaseApp.getInstance().getOptions().getGcmSenderId());
		}
	}

	private static void saveFirebaseToken(@NonNull Context context, @NonNull String firebaseToken) {
		Log.debug("Saving Firebase token");
		String storedToken = getFirebaseTokenFromCache(context);
		OnFirebaseTokenChange(storedToken == null ? "" : storedToken, firebaseToken);
		boolean isUpdatedToken = !TextUtils.isEmpty(storedToken);
		SharedPreferences sharedPreferences = context.getSharedPreferences(PREFS_FILE_FIREBASE, Context.MODE_PRIVATE);
		SharedPreferences.Editor editor = sharedPreferences.edit();
		Log.debug("Firebase token isUpdated : " + isUpdatedToken);
		editor.putBoolean(KEY_IS_UPDATED_TOKEN, isUpdatedToken).apply();
		editor.putBoolean(KEY_IS_REGISTERED, false).apply();
		editor.putString(KEY_FIREBASE_TOKEN, firebaseToken).apply();
	}

	@SuppressWarnings("unused")
	static boolean isFirebaseTokenUpdated(@NonNull Context context) {
		SharedPreferences sharedPreferences = context.getSharedPreferences(PREFS_FILE_FIREBASE, Context.MODE_PRIVATE);
		boolean isUpdated = sharedPreferences.getBoolean(KEY_IS_UPDATED_TOKEN, true);
		Log.debug("Firebase token isUpdatedToken is " + isUpdated);
		return isUpdated;
	}

	public static boolean isFirebaseTokenRegistered(@NonNull Context context) {
		SharedPreferences sharedPreferences = context.getSharedPreferences(PREFS_FILE_FIREBASE, Context.MODE_PRIVATE);
		return sharedPreferences.getBoolean(KEY_IS_REGISTERED, false);
	}

	public static void setFirebaseTokenRegistered(@NonNull Context context, boolean isRegistered) {
		Log.debug("Firebase token isRegistered setting to " + isRegistered);
		SharedPreferences sharedPreferences = context.getSharedPreferences(PREFS_FILE_FIREBASE, Context.MODE_PRIVATE);
		SharedPreferences.Editor editor = sharedPreferences.edit();
		editor.putBoolean(KEY_IS_REGISTERED, isRegistered);
		editor.apply();
	}

	public static void unregisterFirebaseToken(@NonNull Context context) {
		setFirebaseTokenRegistered(context, false);
		SharedPreferences sharedPreferences = context.getSharedPreferences(PREFS_FILE_FIREBASE, Context.MODE_PRIVATE);
		SharedPreferences.Editor editor = sharedPreferences.edit();
		editor.remove(KEY_FIREBASE_TOKEN);
		editor.apply();
		Log.debug("Firebase token cleared");
	}

	private static String getFirebaseTokenFromCache(@NonNull Context context) {
		SharedPreferences sharedPreferences = context.getSharedPreferences(PREFS_FILE_FIREBASE, Context.MODE_PRIVATE);
		return sharedPreferences.getString(KEY_FIREBASE_TOKEN, null);
	}

	private static String getLastFirebaseProjectID(@NonNull Context context)
	{
		SharedPreferences sharedPreferences = context.getSharedPreferences(PREFS_FILE_FIREBASE, Context.MODE_PRIVATE);
		return sharedPreferences.getString(KEY_FIREBASE_PROJECT_ID, null);
	}

	private static void setLastFirebaseProjectID(@NonNull Context context, String projectID)
	{
		SharedPreferences sharedPreferences = context.getSharedPreferences(PREFS_FILE_FIREBASE, Context.MODE_PRIVATE);
		SharedPreferences.Editor editor = sharedPreferences.edit();
		editor.putString(KEY_FIREBASE_PROJECT_ID, projectID);
		editor.apply();
	}

	private static Task<String> queryFirebaseToken(@NonNull Context context, String projectID)
	{
		Task<String> tokenTask = FirebaseMessaging.getInstance().getToken();
		tokenTask.addOnCompleteListener(new OnCompleteListener<String>() {
			@Override
			public void onComplete(@NonNull Task<String> task) {
				if (task.isSuccessful()) {
					String token = task.getResult();
					if(!TextUtils.isEmpty(token)) {
						Log.debug("Firebase token retrieved from Firebase");
						saveFirebaseToken(context, token);
						setLastFirebaseProjectID(context, projectID);
					}
				}
			}
		});
		return tokenTask;
	}

	@Nullable
	public static String getFirebaseToken(@NonNull Context context) {
		EnsureFirebaseIsInitialized(context);

		String projectID = FirebaseApp.getInstance().getOptions().getGcmSenderId();
		String lastProjectID = getLastFirebaseProjectID(context);
		String token;
		if (lastProjectID == null || !projectID.contentEquals(lastProjectID))
		{
			Log.debug("Firebase project changed, querying new token");
			token = null;
			saveFirebaseToken(context, "");
		}
		else
		{
			token = getFirebaseTokenFromCache(context);
		}

		if(TextUtils.isEmpty(token)) {
			Task<String> tokenTask = queryFirebaseToken(context, projectID);
			// wait on task
			try {
				Tasks.await(tokenTask);
				token = getFirebaseTokenFromCache(context);
				token = (token == null) ? "" : token;
			} catch (Exception e) {
				Log.error("Failed to retrieve Firebase token", e);
			}
		} else {
			OnFirebaseTokenChange("", token);
			Log.debug("Firebase token retrieved from cache");

			// Schedule token query anyway in case cached token is stale, do it after OnFirebaseTokenChange to make sure we notify with cached token first
			queryFirebaseToken(context, projectID);
		}

		if (!GameActivity.IS_SHIPPING_CONFIG)
		{
			Log.debug("Firebase token is " + token);
		}

		return token;
	}

	public static void enableFirebaseAutoInit(@NonNull Context context, boolean enableAnalytics) {
		EnsureFirebaseIsInitialized(context);
		FirebaseMessaging.getInstance().setAutoInitEnabled(true);
		if (enableAnalytics) {
			FirebaseAnalytics.getInstance(context).setAnalyticsCollectionEnabled(true);
		}
	}

	public static void EnsureFirebaseIsInitialized(@NonNull Context context) {
		if (FirebaseApp.getApps(context).isEmpty()) {
			FirebaseApp.initializeApp(context);
			if (!GameActivity.IS_SHIPPING_CONFIG)
			{
				Log.debug("Initialized Firebase for project : " + FirebaseApp.getInstance().getOptions().getGcmSenderId());
			}
		}
	}

	public static void deleteFirebaseToken(@NonNull Context context) {
		saveFirebaseToken(context, "");
		FirebaseMessaging.getInstance().deleteToken();
	}

	public static void initializeFirebaseWithCustomFile(@NonNull Context context, String fileName) {
		if (!FirebaseApp.getApps(context).isEmpty())
		{
			Log.error("Failed to initialize Firebase with custom file. Firebase is already initialized!");
			return;
		}

		try
		{
			InputStream inputStream = context.getAssets().open(fileName);
			String result = new BufferedReader(new InputStreamReader(inputStream)).lines().collect(Collectors.joining("\n"));

			/*
				Expected JSON format :
				{
				  "project_info": {
					"project_number": "X",
					"firebase_url": "X",
					"project_id": "X",
					"storage_bucket": "X"
				  },
				  "client": [
					{
					  "client_info": {
						"mobilesdk_app_id": "X"
					  },
					  "api_key": [
						{
						  "current_key": "X"
						}
					  ]
					}
				  ]
				}
			*/

			JSONObject root = new JSONObject(result);
			JSONObject projectInfo = root.getJSONObject("project_info");
			JSONObject client = root.getJSONArray("client").getJSONObject(0);

			String projectNumber = projectInfo.getString("project_number");
			String firebaseUrl = projectInfo.getString("firebase_url");
			String projectId = projectInfo.getString("project_id");
			String storageBucket = projectInfo.getString("storage_bucket");
			String apiKey = client.getJSONArray("api_key").getJSONObject(0).getString("current_key");
			String appID = client.getJSONObject("client_info").getString("mobilesdk_app_id");

			FirebaseOptions.Builder optionsBuilder = new FirebaseOptions.Builder();
			optionsBuilder.setApiKey(apiKey);
			optionsBuilder.setApplicationId(appID);
			optionsBuilder.setDatabaseUrl(firebaseUrl);
			optionsBuilder.setGcmSenderId(projectNumber);
			optionsBuilder.setProjectId(projectId);
			optionsBuilder.setStorageBucket(storageBucket);
			FirebaseOptions options = optionsBuilder.build();

			FirebaseApp.initializeApp(context, options);

			if (!GameActivity.IS_SHIPPING_CONFIG)
			{
				Log.debug("Initialized Firebase with custom file for project : " + FirebaseApp.getInstance().getOptions().getGcmSenderId());
			}
		}
		catch(Exception e)
		{
			Log.error("Failed to initialize Firebase with custom file, using default initialization, exception : ", e);
			FirebaseApp.initializeApp(context);
		}
	}
}
