package com.epicgames.unreal.notifications;

import androidx.annotation.DrawableRes;
import androidx.annotation.NonNull;

public class EpicFirebaseNotificationMeta {
	private final int notificationResId;
	@NonNull
	private final String contentInfo;
	@NonNull
	private final String title;
	@NonNull
	private final String message;
	@NonNull
	private final String type;
	@NonNull
	private final String payload;
	@NonNull
	private final String channelType;
	@NonNull
	private final String channelTitle;
	@NonNull
	private final String channelDescription;

	EpicFirebaseNotificationMeta(@NonNull String contentInfo, @DrawableRes int notificationResId, @NonNull String title,
								 @NonNull String message, @NonNull String type, @NonNull String payload,
								 @NonNull String channelType, @NonNull String channelTitle,
								 @NonNull String channelDescription) {
		this.title = title;
		this.message = message;
		this.contentInfo = contentInfo;
		this.notificationResId = notificationResId;
		this.type = type;
		this.payload = payload;
		this.channelType = channelType;
		this.channelTitle = channelTitle;
		this.channelDescription = channelDescription;
	}
	
	@SuppressWarnings("unused")
	@DrawableRes
	int getNotificationResId() {
		return notificationResId;
	}
	
	@SuppressWarnings("unused")
	@NonNull
	String getContentInfo() {
		return contentInfo;
	}

	@NonNull
	public String getTitle() {
		return title;
	}
	
	@NonNull
	public String getMessage() {
		return message;
	}
	
	@NonNull
	public String getType() {
		return type;
	}
	
	@NonNull
	public String getPayload() {
		return payload;
	}

	@SuppressWarnings("unused")
	@NonNull
	String getChannelType() {
		return channelType;
	}
	
	@SuppressWarnings("unused")
	@NonNull
	String getChannelTitle() {
		return channelTitle;
	}
	
	@SuppressWarnings("unused")
	@NonNull
	String getChannelDescription() {
		return channelDescription;
	}

	@Override
	public String toString() {
		return "EpicFirebaseNotificationMeta{" +
			"notificationResId=" + notificationResId +
			", contentInfo='" + contentInfo + '\'' +
			", title='" + title + '\'' +
			", message='" + message + '\'' +
			", type='" + type + '\'' +
			", payload='" + payload + '\'' +
			", channelType='" + channelType + '\'' +
			", channelTitle='" + channelTitle + '\'' +
			", channelDescription='" + channelDescription + '\'' +
			'}';
	}
}
