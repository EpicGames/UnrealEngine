// Copyright Epic Games, Inc. All Rights Reserved.

#include "Firebase.h"
#include "Misc/CoreDelegates.h"

DEFINE_LOG_CATEGORY(LogFirebase);

#if PLATFORM_IOS
#include "IOS/notifications/EpicFirebaseIOSNotifications.h"
#endif

void IFirebaseModuleInterface::StartupModule()
{
#if PLATFORM_IOS && WITH_IOS_FIREBASE_INTEGRATION
	ApplicationRegisteredForRemoteNotificationsHandle = FCoreDelegates::ApplicationRegisteredForRemoteNotificationsDelegate.AddLambda([](TArray<uint8> Token){
		FFirebaseIOSNotifications::OnAPNSTokenReceived(Token);
	});
#endif
}

void IFirebaseModuleInterface::ShutdownModule()
{
#if PLATFORM_IOS && WITH_IOS_FIREBASE_INTEGRATION
	FCoreDelegates::ApplicationRegisteredForRemoteNotificationsDelegate.Remove(ApplicationRegisteredForRemoteNotificationsHandle);
	ApplicationRegisteredForRemoteNotificationsHandle = {};
#endif
}

class FFirebaseModule : public IFirebaseModuleInterface
{
};

IMPLEMENT_MODULE(FFirebaseModule, Firebase);

#if PLATFORM_ANDROID
#include "Android/AndroidJava.h"
#include "Android/AndroidJavaEnv.h"
#include "Android/AndroidJNI.h"
#include "Async/TaskGraphInterfaces.h"

extern "C"
{
	JNIEXPORT void Java_com_epicgames_unreal_notifications_EpicFirebaseMessagingService_OnFirebaseTokenChange(JNIEnv* jenv, jobject thiz, jstring jPreviousToken, jstring jNewToken)
	{
		if (IFirebaseModuleInterface::Get().OnTokenUpdate.IsBound())
		{
			FString PreviousToken = FJavaHelper::FStringFromParam(jenv, jPreviousToken);
			FString NewToken = FJavaHelper::FStringFromParam(jenv, jNewToken);
			FFunctionGraphTask::CreateAndDispatchWhenReady([PreviousToken, NewToken]()
				{
					IFirebaseModuleInterface::Get().OnTokenUpdate.Broadcast(PreviousToken, NewToken);
				}, TStatId(), NULL, ENamedThreads::GameThread);
		}
	}
}
#endif
