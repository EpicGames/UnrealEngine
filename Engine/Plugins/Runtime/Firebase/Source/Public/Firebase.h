// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Delegates/IDelegateInstance.h"

DECLARE_LOG_CATEGORY_EXTERN(LogFirebase, Log, All);

static const char* FIREBASE_MODULE_NAME = "Firebase";

class IFirebaseModuleInterface : public IModuleInterface
{
private:
	FDelegateHandle ApplicationRegisteredForRemoteNotificationsHandle;

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase - the module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IFirebaseModuleInterface& Get()
	{
		return FModuleManager::LoadModuleChecked<IFirebaseModuleInterface>(FIREBASE_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(FIREBASE_MODULE_NAME);
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	TMulticastDelegate<void(const FString&, const FString&)> OnTokenUpdate;
};
